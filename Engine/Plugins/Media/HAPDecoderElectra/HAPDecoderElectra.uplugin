{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "HAP Decoder for Electra",
	"Description": "Implements video playback of the HAP Codec. HAP is a high performance, high resolution codec that runs on the GPU.",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"Category": "Media Players",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"SupportedTargetPlatforms": [ "Win64", "Mac" ],
	"Modules": [
		{
			"Name": "HAPDecoderElectra",
			"Type": "Runtime",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "Win64", "Mac" ],
			"TargetDenyList": [ "Server" ]
		}
	],
	"Plugins": [
		{
			"Name": "ElectraCodecs",
			"Enabled": true
		}
	]
}
