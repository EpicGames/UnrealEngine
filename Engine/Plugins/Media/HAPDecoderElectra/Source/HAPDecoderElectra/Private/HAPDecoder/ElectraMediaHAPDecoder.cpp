// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraMediaHAPDecoder.h"
#include "HAPDecoderElectraModule.h"
#include "IElectraCodecRegistry.h"
#include "IElectraCodecFactory.h"
#include "Misc/ScopeLock.h"
#include "Templates/SharedPointer.h"
#include "Features/IModularFeatures.h"
#include "Features/IModularFeature.h"
#include "IElectraCodecFactoryModule.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "IElectraDecoder.h"
#include "IElectraDecoderOutputVideo.h"
#include "ElectraDecodersUtils.h"
#include "Utils/ElectraBitstreamProcessorDefault.h"

#include "ElectraDecodersPlatformResources.h"
#include COMPILED_PLATFORM_HEADER(ElectraTextureSampleGPUBufferHelper.h)

#include "hap.h"

#define ERRCODE_INTERNAL_NO_ERROR							0
#define ERRCODE_INTERNAL_ALREADY_CLOSED						1
#define ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT				2

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

class FVideoDecoderHAPElectra;


class FVideoDecoderOutputHAPElectra : public IElectraDecoderVideoOutput, public IElectraDecoderVideoOutputImageBuffers
{
public:
	virtual ~FVideoDecoderOutputHAPElectra()
	{
	}

	FTimespan GetPTS() const override
	{
		return PTS;
	}
	uint64 GetUserValue() const override
	{
		return UserValue;
	}

	EOutputType GetOutputType() const
	{
		return EOutputType::Output;
	}

	int32 GetWidth() const override
	{
		return Width - Crop.Left - Crop.Right;
	}
	int32 GetHeight() const override
	{
		return Height - Crop.Top - Crop.Bottom;
	}
	int32 GetDecodedWidth() const override
	{
		return Width;
	}
	int32 GetDecodedHeight() const override
	{
		return Height;
	}
	FElectraVideoDecoderOutputCropValues GetCropValues() const override
	{
		return Crop;
	}
	int32 GetAspectRatioW() const override
	{
		return AspectW;
	}
	int32 GetAspectRatioH() const override
	{
		return AspectH;
	}
	int32 GetFrameRateNumerator() const override
	{
		return FrameRateN;
	}
	int32 GetFrameRateDenominator() const override
	{
		return FrameRateD;
	}
	int32 GetNumberOfBits() const override
	{
		return NumBits;
	}
	void GetExtraValues(TMap<FString, FVariant>& OutExtraValues) const override
	{
		OutExtraValues = ExtraValues;
	}
	void* GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType InTypeOfHandle) const override
	{
		if (InTypeOfHandle == EElectraDecoderPlatformOutputHandleType::ImageBuffers)
		{
			return static_cast<IElectraDecoderVideoOutputImageBuffers*>(const_cast<FVideoDecoderOutputHAPElectra*>(this));
		}
		return nullptr;
	}
	IElectraDecoderVideoOutput::EImageCopyResult CopyPlatformImage(IElectraDecoderVideoOutputCopyResources* InCopyResources) const override
	{
		return IElectraDecoderVideoOutput::EImageCopyResult::NotSupported;
	}

	// Methods from IElectraDecoderVideoOutputImageBuffers
	uint32 GetCodec4CC() const override
	{
		return Codec4CC;
	}
	int32 GetNumberOfBuffers() const override
	{
		return NumBuffers;
	}
	TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> GetBufferDataByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return Buffer[Buffer_Color].Buffer;
		}
		else if (InBufferIndex == 1)
		{
			return Buffer[Buffer_Alpha].Buffer;
		}
		return nullptr;
	}
	void* GetBufferTextureByIndex(int32 InBufferIndex) const override
	{
#if ELECTRA_MEDIAGPUBUFFER_DX12
		if (InBufferIndex == 0)
		{
			return Buffer[Buffer_Color].GPUBuffer.Resource.GetReference();
		}
		if (InBufferIndex == 1)
		{
			return Buffer[Buffer_Alpha].GPUBuffer.Resource.GetReference();
		}
#endif
		return nullptr;
	}
	virtual bool GetBufferTextureSyncByIndex(int32 InBufferIndex, FElectraDecoderOutputSync& SyncObject) const override
	{
#if ELECTRA_MEDIAGPUBUFFER_DX12
		if (InBufferIndex == 0)
		{
			SyncObject = { Buffer[Buffer_Color].GPUBuffer.Fence.GetReference(), Buffer[Buffer_Color].GPUBuffer.FenceValue, nullptr };
			return true;
		}
		if (InBufferIndex == 1)
		{
			SyncObject = { Buffer[Buffer_Alpha].GPUBuffer.Fence.GetReference(), Buffer[Buffer_Alpha].GPUBuffer.FenceValue, nullptr };
			return true;
		}
#endif
		return false;
	}

	EPixelFormat GetBufferFormatByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return Buffer[Buffer_Color].BufferFormat;
		}
		else if (InBufferIndex == 1)
		{
			return Buffer[Buffer_Alpha].BufferFormat;
		}
		return EPixelFormat::PF_Unknown;
	}
	EElectraTextureSamplePixelEncoding GetBufferEncodingByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return Buffer[Buffer_Color].BufferEncoding;
		}
		else if (InBufferIndex == 1)
		{
			return Buffer[Buffer_Alpha].BufferEncoding;
		}
		return EElectraTextureSamplePixelEncoding::Native;
	}
	int32 GetBufferPitchByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return Buffer[Buffer_Color].Pitch;
		}
		else if (InBufferIndex == 1)
		{
			return Buffer[Buffer_Alpha].Pitch;
		}
		return 0;
	}

public:
	FTimespan PTS;
	uint64 UserValue = 0;

	FElectraVideoDecoderOutputCropValues Crop;
	int32 Width = 0;
	int32 Height = 0;
	int32 NumBits = 0;
	int32 AspectW = 1;
	int32 AspectH = 1;
	int32 FrameRateN = 0;
	int32 FrameRateD = 0;
	int32 PixelFormat = 0;
	TMap<FString, FVariant> ExtraValues;

	uint32 Codec4CC = 0;
	int32 NumBuffers = 0;

	enum
	{
		Buffer_Color = 0,
		Buffer_Alpha,
		Buffer_Max
	};

	struct FBufferInfo
	{
		TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> Buffer;
		EPixelFormat BufferFormat;
		EElectraTextureSamplePixelEncoding BufferEncoding;
		int32 Pitch = 0;
#if ELECTRA_MEDIAGPUBUFFER_DX12
		FElectraMediaDecoderOutputBufferPool_DX12::FOutputData GPUBuffer;
#endif
	};
	FBufferInfo Buffer[Buffer_Max];
};


class FVideoDecoderHAPElectra : public IElectraDecoder
{
public:
	static void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions)
	{
		OutOptions.Emplace(IElectraDecoderFeature::MinimumNumberOfOutputFrames, FVariant((int32)5));
		OutOptions.Emplace(IElectraDecoderFeature::IsAdaptive, FVariant(true));
	}

	FVideoDecoderHAPElectra(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);
	virtual ~FVideoDecoderHAPElectra();

	IElectraDecoder::EType GetType() const override
	{
		return IElectraDecoder::EType::Video;
	}

	void GetFeatures(TMap<FString, FVariant>& OutFeatures) const override;

	FError GetError() const override;

	void Close() override;
	IElectraDecoder::ECSDCompatibility IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions) override;
	bool ResetToCleanStart() override;

	EDecoderError DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions) override;
	EDecoderError SendEndOfData() override;
	EDecoderError Flush() override;
	EOutputStatus HaveOutput() override;
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> GetOutput() override;

	TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> CreateBitstreamProcessor() override
	{ return FElectraDecoderBitstreamProcessorDefault::Create(); }

	void Suspend() override
	{ }
	void Resume() override
	{ }

private:
	static void HapDecodeCallback(HapDecodeWorkFunction InFunction, void* InParameter, unsigned int InCount, void* InInfo)
	{
		for(unsigned int i=0; i<InCount; ++i)
		{
			InFunction(InParameter, i);
		}
	}

	bool PostError(int32 ApiReturnValue, FString Message, int32 Code);

	int32 DisplayWidth = 0;
	int32 DisplayHeight = 0;
	int32 DecodedWidth = 0;
	int32 DecodedHeight = 0;
	int32 AspectW = 0;
	int32 AspectH = 0;

	IElectraDecoder::FError LastError;

	Electra::FCodecTypeFormat InitialCodecFormat;
	TMap<FString, FVariant> InitialCreationOptions;

	TSharedPtr<FVideoDecoderOutputHAPElectra, ESPMode::ThreadSafe> CurrentOutput;
	bool bFlushPending = false;

	uint32 MaxOutputBuffers;
	struct FBufferInstanceInfo
	{
		uint32 AllocSize = 0;
		uint32 AllocBlockSize = 0;
#if ELECTRA_MEDIAGPUBUFFER_DX12
		mutable TSharedPtr<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe> D3D12ResourcePool;
#endif
	};
	FBufferInstanceInfo InstanceBufferData[FVideoDecoderOutputHAPElectra::Buffer_Max];
};


/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

class FHAPVideoDecoderElectraFactory : public IElectraCodecFactory, public IElectraCodecFactory::IProviderInformation, public IElectraCodecModularFeature, public TSharedFromThis<FHAPVideoDecoderElectraFactory, ESPMode::ThreadSafe>
{
public:
	virtual ~FHAPVideoDecoderElectraFactory()
	{ }

	void GetListOfFactories(TArray<TWeakPtr<IElectraCodecFactory, ESPMode::ThreadSafe>>& OutCodecFactories) override
	{
		OutCodecFactories.Add(AsShared());
	}

	void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions) const override
	{
		FVideoDecoderHAPElectra::GetConfigurationOptions(OutOptions);
	}

	int32 SupportsDecoding(TMap<FString, FVariant>& OutFormatInfo, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) const override
	{
		if (InCodecFormat.Type != Electra::FCodecTypeFormat::EType::Video || !InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>())
		{
			return 0;
		}
		bool bIsSupported = true;
		FString Name;
		// See: https://github.com/Vidvox/hap/blob/master/documentation/HapVideoDRAFT.md#names-and-identifiers
		switch(InCodecFormat.FourCC)
		{
			case ElectraDecodersUtil::Make4CC('H','a','p','1'):
			{
				Name = TEXT("Hap");
				break;
			}
			case ElectraDecodersUtil::Make4CC('H','a','p','5'):
			{
				Name = TEXT("Hap Alpha");
				break;
			}
			case ElectraDecodersUtil::Make4CC('H','a','p','Y'):
			{
				Name = TEXT("Hap Q");
				break;
			}
			case ElectraDecodersUtil::Make4CC('H','a','p','M'):
			{
				Name = TEXT("Hap Q Alpha");
				break;
			}
			case ElectraDecodersUtil::Make4CC('H','a','p','7'):
			{
				Name = TEXT("Hap R");
				break;
			}
			case ElectraDecodersUtil::Make4CC('H','a','p','H'):
			{
				Name = TEXT("Hap HDR");
				break;
			}
			default:
			{
				bIsSupported = false;
				break;
			}
		}
		if (bIsSupported)
		{
			if (InCodecFormat.HumanReadableFormatInfo.IsEmpty())
			{
				OutFormatInfo.Emplace(IElectraDecoderFormatInfo::HumanReadableFormatName, FVariant(Name));
			}
			else
			{
				OutFormatInfo.Emplace(IElectraDecoderFormatInfo::HumanReadableFormatName, InCodecFormat.HumanReadableFormatInfo);
			}
			OutFormatInfo.Emplace(IElectraDecoderFormatInfo::IsEveryFrameKeyframe, FVariant(true));
			return 1;
		}
		return 0;
	}

	TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> CreateDecoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) override
	{
		return InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>() ? MakeShared<FVideoDecoderHAPElectra, ESPMode::ThreadSafe>(InCodecFormat, InAdditionalOptions) : TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe>();
	}


	const IProviderInformation& GetProviderInformation() const override
	{ return *this; }
	FString GetName() const override
	{ return FString(TEXT("HAPlib")); }
	FString GetVersion() const override
	{ return(FString()); }
	FString GetImplementation() const override
	{ return(FString()); }
	FString GetVendor() const override
	{ return(FString()); }

	static TSharedPtr<FHAPVideoDecoderElectraFactory, ESPMode::ThreadSafe> Self;
};
TSharedPtr<FHAPVideoDecoderElectraFactory, ESPMode::ThreadSafe> FHAPVideoDecoderElectraFactory::Self;

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

void FElectraMediaHAPDecoder::Startup()
{
	// Make sure the codec factory module has been loaded.
	FModuleManager::Get().LoadModule(TEXT("ElectraCodecFactory"));

	// Create an instance of the factory, which is also the modular feature.
	check(!FHAPVideoDecoderElectraFactory::Self.IsValid());
	FHAPVideoDecoderElectraFactory::Self = MakeShared<FHAPVideoDecoderElectraFactory, ESPMode::ThreadSafe>();
	// Register as modular feature.
	IModularFeatures::Get().RegisterModularFeature(IElectraCodecFactoryModule::GetModularFeatureName(), FHAPVideoDecoderElectraFactory::Self.Get());
}

void FElectraMediaHAPDecoder::Shutdown()
{
	IModularFeatures::Get().UnregisterModularFeature(IElectraCodecFactoryModule::GetModularFeatureName(), FHAPVideoDecoderElectraFactory::Self.Get());
	FHAPVideoDecoderElectraFactory::Self.Reset();
}

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

FVideoDecoderHAPElectra::FVideoDecoderHAPElectra(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
	: InitialCodecFormat(InCodecFormat)
	, InitialCreationOptions(InAdditionalOptions)
{
	const Electra::FCodecTypeFormat::FVideo& vid(InitialCodecFormat.Properties.Get<Electra::FCodecTypeFormat::FVideo>());
	DisplayWidth = (int32) vid.Width;
	DisplayHeight = (int32) vid.Height;
	// All HAP formats have 4x4 pixel blocks.
	DecodedWidth = Align(DisplayWidth, 4);
	DecodedHeight = Align(DisplayHeight, 4);
	AspectW = (int32) vid.AspectRatioW;
	AspectH = (int32) vid.AspectRatioH;

	switch(InitialCodecFormat.FourCC)
	{
		case ElectraDecodersUtil::Make4CC('H','a','p','1'):
		case ElectraDecodersUtil::Make4CC('H','a','p','A'):
		{
			InstanceBufferData[FVideoDecoderOutputHAPElectra::Buffer_Color].AllocSize = DecodedWidth * DecodedHeight / 2;
			InstanceBufferData[FVideoDecoderOutputHAPElectra::Buffer_Alpha].AllocSize = DecodedWidth * DecodedHeight / 2;
			InstanceBufferData[FVideoDecoderOutputHAPElectra::Buffer_Color].AllocBlockSize = 8;
			InstanceBufferData[FVideoDecoderOutputHAPElectra::Buffer_Alpha].AllocBlockSize = 8;
			break;
		}
		case ElectraDecodersUtil::Make4CC('H','a','p','Y'):
		case ElectraDecodersUtil::Make4CC('H','a','p','5'):
		{
			InstanceBufferData[FVideoDecoderOutputHAPElectra::Buffer_Color].AllocSize = DecodedWidth * DecodedHeight;
			InstanceBufferData[FVideoDecoderOutputHAPElectra::Buffer_Alpha].AllocSize = DecodedWidth * DecodedHeight;
			InstanceBufferData[FVideoDecoderOutputHAPElectra::Buffer_Color].AllocBlockSize = 16;
			InstanceBufferData[FVideoDecoderOutputHAPElectra::Buffer_Alpha].AllocBlockSize = 16;
			break;
		}
		default:
		{
			InstanceBufferData[FVideoDecoderOutputHAPElectra::Buffer_Color].AllocSize = DecodedWidth * DecodedHeight * 4;
			InstanceBufferData[FVideoDecoderOutputHAPElectra::Buffer_Alpha].AllocSize = DecodedWidth * DecodedHeight * 4;
			InstanceBufferData[FVideoDecoderOutputHAPElectra::Buffer_Color].AllocBlockSize = 0;
			InstanceBufferData[FVideoDecoderOutputHAPElectra::Buffer_Alpha].AllocBlockSize = 0;
			break;
		}
	}

	MaxOutputBuffers = (uint32)ElectraDecodersUtil::GetVariantValueSafeU64(InitialCreationOptions, TEXT("max_output_buffers"), 5);
	MaxOutputBuffers += kElectraDecoderPipelineExtraFrames;
}

FVideoDecoderHAPElectra::~FVideoDecoderHAPElectra()
{
	// Close() must have been called already!
	check(LastError.Code == ERRCODE_INTERNAL_ALREADY_CLOSED);
	// We do it nonetheless...
	Close();
}

void FVideoDecoderHAPElectra::GetFeatures(TMap<FString, FVariant>& OutFeatures) const
{
	GetConfigurationOptions(OutFeatures);
}

IElectraDecoder::FError FVideoDecoderHAPElectra::GetError() const
{
	return LastError;
}

bool FVideoDecoderHAPElectra::PostError(int32 ApiReturnValue, FString Message, int32 Code)
{
	LastError.Code = Code;
	LastError.SdkCode = ApiReturnValue;
	LastError.Message = MoveTemp(Message);
	return false;
}

void FVideoDecoderHAPElectra::Close()
{
	ResetToCleanStart();
	// Set the error state that all subsequent calls will fail.
	PostError(0, TEXT("Already closed"), ERRCODE_INTERNAL_ALREADY_CLOSED);
}

IElectraDecoder::ECSDCompatibility FVideoDecoderHAPElectra::IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions)
{
	return IElectraDecoder::ECSDCompatibility::Compatible;
}

bool FVideoDecoderHAPElectra::ResetToCleanStart()
{
	bFlushPending = false;
	CurrentOutput.Reset();
	return true;
}

static bool ConvertHapTextureFormat(HapTextureFormat HapFmt, EPixelFormat& Format, EElectraTextureSamplePixelEncoding& Encoding)
{
	switch (HapFmt)
	{
		case HapTextureFormat_RGB_DXT1:
		{
			Format = EPixelFormat::PF_DXT1;
			Encoding = EElectraTextureSamplePixelEncoding::Native;
			break;
		}
		case HapTextureFormat_RGBA_DXT5:
		{
			Format = EPixelFormat::PF_DXT5;
			Encoding = EElectraTextureSamplePixelEncoding::Native;
			break;
		}
		case HapTextureFormat_YCoCg_DXT5:
		{
			Format = EPixelFormat::PF_DXT5;
			Encoding = EElectraTextureSamplePixelEncoding::YCoCg;
			break;
		}
		case HapTextureFormat_A_RGTC1:
		{
			Format = EPixelFormat::PF_BC4;
			Encoding = EElectraTextureSamplePixelEncoding::Native;
			break;
		}
		case HapTextureFormat_RGBA_BPTC_UNORM:
		case HapTextureFormat_RGB_BPTC_UNSIGNED_FLOAT:
		case HapTextureFormat_RGB_BPTC_SIGNED_FLOAT:
		default:
		{
			Format = EPixelFormat::PF_Unknown;
			Encoding = EElectraTextureSamplePixelEncoding::Native;
			break;
		}
	}
	return Format != EPixelFormat::PF_Unknown;
}

static uint32 GetImageBufferPitch(HapTextureFormat HapFmt, uint32 Width)
{
	uint32 Pitch = 0;
	switch (HapFmt)
	{
		case HapTextureFormat_RGB_DXT1:
		case HapTextureFormat_A_RGTC1:
			Pitch = ((Width + 3) / 4) * 8;		// 4 pixel wide blocks with 8 bytes
			break;
		case HapTextureFormat_RGBA_DXT5:
		case HapTextureFormat_YCoCg_DXT5:
			Pitch = ((Width + 3) / 4) * 16;		// 4 pixel wide blocks with 16 bytes
			break;
		case HapTextureFormat_RGBA_BPTC_UNORM:
		case HapTextureFormat_RGB_BPTC_UNSIGNED_FLOAT:
		case HapTextureFormat_RGB_BPTC_SIGNED_FLOAT:
		default:
			break;
	}
	return Pitch;
}

IElectraDecoder::EDecoderError FVideoDecoderHAPElectra::DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Can not feed new input until draining has finished.
	if (bFlushPending)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// If there is pending output it is very likely that decoding this access unit would also generate output.
	// Since that would result in loss of the pending output we return now.
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}

#if ELECTRA_MEDIAGPUBUFFER_DX12
	// If we will create a new resource pool or we have still buffers in an existing one, we can proceed, else we'd have no resources to output the data
	for (uint32 BufIdx = 0; BufIdx < FVideoDecoderOutputHAPElectra::Buffer_Max; ++BufIdx)
	{
		if (InstanceBufferData[BufIdx].D3D12ResourcePool.IsValid() && !InstanceBufferData[BufIdx].D3D12ResourcePool->BufferAvailable())
		{
			return IElectraDecoder::EDecoderError::NoBuffer;
		}
	}
#endif

	// Decode data. This immediately produces a new output frame.
	if (InInputAccessUnit.Data && InInputAccessUnit.DataSize)
	{
		unsigned int TextureCount = 0;
		HapResult Result = static_cast<HapResult>(HapGetFrameTextureCount(InInputAccessUnit.Data, static_cast<unsigned long>(InInputAccessUnit.DataSize), &TextureCount));
		if (Result != HapResult_No_Error)
		{
			PostError(Result, TEXT("HapGetFrameTextureCount() failed"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}

		void* PlatformDevice = nullptr;
		int32 PlatformDeviceVersion = 0;
		bool bUseGPUBuffers = false;
#if ELECTRA_MEDIAGPUBUFFER_DX12
		FElectraDecodersPlatformResources::GetD3DDeviceAndVersion(&PlatformDevice, &PlatformDeviceVersion);
		bUseGPUBuffers = (PlatformDevice && PlatformDeviceVersion >= 12000);
#endif

		TSharedPtr<FVideoDecoderOutputHAPElectra, ESPMode::ThreadSafe> NewOutput = MakeShared<FVideoDecoderOutputHAPElectra>();
		NewOutput->PTS = InInputAccessUnit.PTS;
		NewOutput->UserValue = InInputAccessUnit.UserValue;

		NewOutput->Width = DecodedWidth;
		NewOutput->Height = DecodedHeight;
		NewOutput->Crop.Right = DecodedWidth - DisplayWidth;
		NewOutput->Crop.Bottom = DecodedHeight - DisplayHeight;
		if (AspectW && AspectH)
		{
			NewOutput->AspectW = AspectW;
			NewOutput->AspectH = AspectH;
		}
// These actually depend on the format, but we don't support the latest HAP formats at the moment.
		NewOutput->NumBits = 8;
		NewOutput->PixelFormat = 0;

		NewOutput->ExtraValues.Emplace(TEXT("codec"), FVariant(TEXT("hap")));
		NewOutput->ExtraValues.Emplace(TEXT("codec_4cc"), FVariant(InitialCodecFormat.FourCC));

		// Setup all buffers / textures (we clamp to what out max number is)
		uint32 NumBuffers = FMath::Min(TextureCount, (uint32)FVideoDecoderOutputHAPElectra::Buffer_Max);
		for (uint32 BufIdx = 0; BufIdx < NumBuffers; ++BufIdx)
		{
			auto& BufferInfo = NewOutput->Buffer[BufIdx];

			uint8* BufferAddr = nullptr;
			uint32 BufferPitch = 0;
			uint32 ResourcePitch = 0;
			uint32 BlockSizeX = InstanceBufferData[BufIdx].AllocBlockSize ? 4 : 1;
			uint32 BlockSizeY = InstanceBufferData[BufIdx].AllocBlockSize ? 4 : 1;
			uint32 BlockBytes = InstanceBufferData[BufIdx].AllocBlockSize ? InstanceBufferData[BufIdx].AllocBlockSize : 4;

#if ELECTRA_MEDIAGPUBUFFER_DX12
			TSharedPtr<TArray<uint8>, ESPMode::ThreadSafe> TempBuffer;
			if (bUseGPUBuffers)
			{
				TRefCountPtr D3D12Device(static_cast<ID3D12Device*>(PlatformDevice));
				HRESULT HResult;
				FString HResultMsg;

				// Create the resource pool as needed...
				if (!InstanceBufferData[BufIdx].D3D12ResourcePool.IsValid())
				{
					// We always expect to get BCx style data here, but allow for a default RGBA8 setup (as does the initialization code)
					InstanceBufferData[BufIdx].D3D12ResourcePool = MakeShared<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe>(D3D12Device, MaxOutputBuffers, DecodedWidth / BlockSizeX, DecodedHeight / BlockSizeY, BlockBytes);
				}

				if (InstanceBufferData[BufIdx].D3D12ResourcePool->AllocateOutputDataAsBuffer(HResult, HResultMsg, BufferInfo.GPUBuffer, ResourcePitch))
				{
					// Make the decoder write to a cached, temp buffer (otherwise performance is quite bad)
					TempBuffer = MakeShared<TArray<uint8>, ESPMode::ThreadSafe>();
					if (!TempBuffer.IsValid())
					{
						PostError(Result, TEXT("Could not allocate HAP decoder output buffer"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
						return IElectraDecoder::EDecoderError::Error;
					}
					TempBuffer->AddUninitialized(InstanceBufferData[BufIdx].AllocSize);
					BufferAddr = TempBuffer->GetData();
					if (BufferAddr == nullptr)
					{
						PostError(Result, TEXT("Could not allocate HAP decoder output buffer"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
						return IElectraDecoder::EDecoderError::Error;
					}
					BufferPitch = (DecodedWidth / BlockSizeX) * BlockBytes;
				}
				else
				{
					PostError(Result, FString::Printf(TEXT("Could not allocate D3D12 buffer (%s)"), *HResultMsg), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
					return IElectraDecoder::EDecoderError::Error;
				}
			}
			else
#endif
			{
				BufferInfo.Buffer = MakeShared<TArray64<uint8>, ESPMode::ThreadSafe>();
				BufferInfo.Buffer->AddUninitialized(InstanceBufferData[BufIdx].AllocSize);
				BufferAddr = BufferInfo.Buffer->GetData();
				if (BufferAddr == nullptr)
				{
					PostError(Result, TEXT("Could not allocate HAP decoder output buffer"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
					return IElectraDecoder::EDecoderError::Error;
				}
			}

			unsigned long BufferBytesUsed = 0;
			unsigned int BufferTextureFormat = 0;

			Result = static_cast<HapResult>(HapDecode(InInputAccessUnit.Data, static_cast<unsigned long>(InInputAccessUnit.DataSize), 0, HapDecodeCallback, nullptr,
													  BufferAddr, InstanceBufferData[BufIdx].AllocSize, &BufferBytesUsed, &BufferTextureFormat));

#if ELECTRA_MEDIAGPUBUFFER_DX12
			if (bUseGPUBuffers)
			{
				if (Result == HapResult_No_Error)
				{
					uint8* DX12BufferAddr;
					HRESULT Res = BufferInfo.GPUBuffer.Resource->Map(0, nullptr, (void**)&DX12BufferAddr);
					check(SUCCEEDED(Res));

					FElectraMediaDecoderOutputBufferPool_DX12::CopyWithPitchAdjust(DX12BufferAddr, ResourcePitch, BufferAddr, BufferPitch, DecodedHeight / BlockSizeY);

					BufferInfo.GPUBuffer.Resource->Unmap(0, nullptr);

					BufferInfo.GPUBuffer.Fence->Signal(BufferInfo.GPUBuffer.FenceValue);

					BufferInfo.Pitch = ResourcePitch;
				}
				TempBuffer.Reset();
			}
			else
#endif
			{
				if (Result == HapResult_No_Error)
				{
					BufferInfo.Buffer->SetNumUnsafeInternal((int32)BufferBytesUsed);
					BufferInfo.Pitch = GetImageBufferPitch(static_cast<HapTextureFormat>(BufferTextureFormat), NewOutput->Width);
				}
			}

			if (Result != HapResult_No_Error)
			{
				PostError(Result, TEXT("HapDecode() failed"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
				return IElectraDecoder::EDecoderError::Error;
			}

			ConvertHapTextureFormat(static_cast<HapTextureFormat>(BufferTextureFormat), BufferInfo.BufferFormat, BufferInfo.BufferEncoding);

			NewOutput->NumBuffers = (int32)TextureCount;
			NewOutput->Codec4CC = InitialCodecFormat.FourCC;
		}

		CurrentOutput = MoveTemp(NewOutput);
	}
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FVideoDecoderHAPElectra::SendEndOfData()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	// Already draining?
	if (bFlushPending)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}
	bFlushPending = true;
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FVideoDecoderHAPElectra::Flush()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	ResetToCleanStart();
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EOutputStatus FVideoDecoderHAPElectra::HaveOutput()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EOutputStatus::Error;
	}
	// Have output?
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EOutputStatus::Available;
	}
	// Pending flush?
	if (bFlushPending)
	{
		bFlushPending = false;
		return IElectraDecoder::EOutputStatus::EndOfData;
	}
	return IElectraDecoder::EOutputStatus::NeedInput;
}

TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> FVideoDecoderHAPElectra::GetOutput()
{
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> Out = CurrentOutput;
	CurrentOutput.Reset();
	return Out;
}
