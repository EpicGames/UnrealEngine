// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAPDecoderElectraModule.h"
#include "HAPDecoder/ElectraMediaHAPDecoder.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "HAPDecoderElectraModule"

DEFINE_LOG_CATEGORY(LogHAPElectraDecoder);

class FHAPElectraDecoderModule : public IModuleInterface
{
public:
	void StartupModule() override
	{
		FElectraMediaHAPDecoder::Startup();
	}

	void ShutdownModule() override
	{
		FElectraMediaHAPDecoder::Shutdown();
	}

	bool SupportsDynamicReloading() override
	{
		// Codec could still be in use
		return false;
	}
};

IMPLEMENT_MODULE(FHAPElectraDecoderModule, HAPDecoderElectra);

#undef LOCTEXT_NAMESPACE
